{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Optional Mobile Features Blueprint Library",
	"Description": "Gives blueprint access to Sound Volume, Battery Charge Level, and System Temperature for Android and iOS devices",
	"Category": "Mobile",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "OptionalMobileFeaturesBPLibrary",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"IsExperimentalVersion": false
}