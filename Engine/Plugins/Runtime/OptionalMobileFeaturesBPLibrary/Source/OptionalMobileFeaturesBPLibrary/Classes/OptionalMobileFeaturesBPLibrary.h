// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "OptionalMobileFeaturesBPLibrary.generated.h"

UCLASS()
class UOptionalMobileFeaturesBPLibrary :
	public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Returns the current volume state of the device in a range of 0-100 (%)  */
	UFUNCTION(BlueprintCallable, Category="Mobile")
	static int GetVolumeState();

	/** Returns the current battery level of the device in a range of [0, 100] */
	UFUNCTION(BlueprintCallable, Category = "Mobile")
	static int GetBatteryLevel();

	/** Returns the device's temperature, in Celsius */
	UFUNCTION(BlueprintCallable, Category = "Mobile")
	static float GetBatteryTemperature();

	/** Returns if headphones are plugged into the device */
	UFUNCTION(BlueprintCallable, Category = "Mobile")
	static bool AreHeadphonesPluggedIn();

};
