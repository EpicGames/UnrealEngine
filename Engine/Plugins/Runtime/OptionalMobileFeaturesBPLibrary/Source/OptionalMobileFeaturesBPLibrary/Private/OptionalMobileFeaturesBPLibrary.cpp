// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptionalMobileFeaturesBPLibrary.h"

DEFINE_LOG_CATEGORY_STATIC(LogOptionalMobileBPLib, Display, All);

UOptionalMobileFeaturesBPLibrary::UOptionalMobileFeaturesBPLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int UOptionalMobileFeaturesBPLibrary::GetVolumeState()
{
#if PLATFORM_ANDROID
	//FAndroidMisc::GetVolumeState returns 0-15, scale to 0-100
	int baseVolume = FAndroidMisc::GetVolumeState();
	int scaledVolume = (baseVolume * 100) / 15;
	return scaledVolume;
#elif PLATFORM_IOS
	return FPlatformMisc::GetDeviceVolume();
#else
	return 0;
#endif
}

int UOptionalMobileFeaturesBPLibrary::GetBatteryLevel()
{
#if PLATFORM_ANDROID
	return FAndroidMisc::GetBatteryState().Level;
#elif PLATFORM_IOS
	return FIOSPlatformMisc::GetBatteryLevel();
#else
	return 0;
#endif
}

float UOptionalMobileFeaturesBPLibrary::GetBatteryTemperature()
{
#if PLATFORM_ANDROID
	return FAndroidMisc::GetBatteryState().Temperature;
#elif PLATFORM_IOS
	//no current public API for this, add here if that changes
	return 0.0f;
#else
	return 0.0f;
#endif
}

bool UOptionalMobileFeaturesBPLibrary::AreHeadphonesPluggedIn()
{
#if PLATFORM_ANDROID
	return FAndroidMisc::AreHeadPhonesPluggedIn();
#elif PLATFORM_IOS
	return FIOSPlatformMisc::AreHeadphonesPluggedIn();
#else
	return false;
#endif
}
