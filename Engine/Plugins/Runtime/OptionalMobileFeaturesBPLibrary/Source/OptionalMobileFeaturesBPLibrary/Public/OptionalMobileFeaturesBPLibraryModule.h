// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

/**
 * The public interface to this module
 */
class FOptionalMobileFeaturesBPLibraryModule :
	public IModuleInterface
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FOptionalMobileFeaturesBPLibraryModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FOptionalMobileFeaturesBPLibraryModule >( "OptionalMobileFeaturesBPLibrary" );
	}

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

